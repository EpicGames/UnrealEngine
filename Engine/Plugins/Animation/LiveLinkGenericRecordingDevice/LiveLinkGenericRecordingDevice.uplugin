{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LiveLinkGenericRecordingDevice",
	"Description": "Provides a generic recording device that can be used by Python scripts for Live Link Hub. It is hidden by default as the classes are abstract.",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
    "Hidden": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "LiveLinkGenericRecordingDevice",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "LiveLinkDevice",
			"Enabled": true
		}
	]
}
