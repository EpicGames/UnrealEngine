// Copyright Epic Games, Inc. All Rights Reserved.
 
using UnrealBuildTool;

public class LiveLinkGenericRecordingDevice : ModuleRules
{
	public LiveLinkGenericRecordingDevice(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePaths.AddRange(
			new string[] {
				// Add public include paths here
			}
		);

		PrivateIncludePaths.AddRange(
			new string[] {
				// Add private include paths here
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"LiveLinkDevice",
				"Projects"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// Add dynamically loaded modules here
			}
		);
	}
}
