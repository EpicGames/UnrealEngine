// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkGenericRecordingDevice.h"

TSubclassOf<ULiveLinkDeviceSettings> ULiveLinkGenericRecordingDevice::GetSettingsClass() const
{
	return SettingsClass ? SettingsClass : TSubclassOf<ULiveLinkDeviceSettings>(ULiveLinkGenericRecordingDeviceSettings::StaticClass());
}

FText ULiveLinkGenericRecordingDevice::GetDisplayName() const
{
	// Delegate to BlueprintNativeEvent version
	return GetDisplayNameHelper();
}

 EDeviceHealth ULiveLinkGenericRecordingDevice::GetDeviceHealth() const
{
	// Delegate to BlueprintNativeEvent version
	return GetDeviceHealthHelper();
}

FText ULiveLinkGenericRecordingDevice::GetHealthText() const
{
	// Delegate to BlueprintNativeEvent version
	return GetHealthTextHelper();
}

FText ULiveLinkGenericRecordingDevice::GetDisplayNameHelper_Implementation() const
{
	return FText::FromString(GetDeviceSettings()->DisplayName);
}

EDeviceHealth ULiveLinkGenericRecordingDevice::GetDeviceHealthHelper_Implementation() const
{
	return EDeviceHealth::Nominal;
}

FText ULiveLinkGenericRecordingDevice::GetHealthTextHelper_Implementation() const
{
	return FText::FromString(TEXT("OK"));
}

bool ULiveLinkGenericRecordingDevice::StartRecording_Implementation()
{
	bIsRecording = true;
	return true;
}

bool ULiveLinkGenericRecordingDevice::StopRecording_Implementation() 
{
	bIsRecording = false;
	return true;
}

bool ULiveLinkGenericRecordingDevice::IsRecording_Implementation() const
{
	return bIsRecording;
}


