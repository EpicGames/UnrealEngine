// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDevice.h"
#include "LiveLinkDeviceCapability_Connection.h"
#include "LiveLinkDeviceCapability_Recording.h"
#include "LiveLinkGenericRecordingDevice.generated.h"

UCLASS(Blueprintable)
class ULiveLinkGenericRecordingDeviceSettings : public ULiveLinkDeviceSettings
{
    GENERATED_BODY()

public:

	ULiveLinkGenericRecordingDeviceSettings()
	{
		DisplayName = TEXT("Python Recording Device");
	}
};

UCLASS(Abstract, Blueprintable)
class ULiveLinkGenericRecordingDevice : public ULiveLinkDevice
    , public ILiveLinkDeviceCapability_Recording  // Implement the recording interface
{
    GENERATED_BODY()

public:
    //~ Begin ULiveLinkDevice interface (pure virtuals)
    // These delegate to BlueprintNativeEvent versions that Python can override
    virtual TSubclassOf<ULiveLinkDeviceSettings> GetSettingsClass() const override;
    virtual FText GetDisplayName() const override;
    virtual EDeviceHealth GetDeviceHealth() const override;
    virtual FText GetHealthText() const override;
    //~ End ULiveLinkDevice interface

    // BlueprintNativeEvent versions that Python scripts can override
    UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category="Live Link Device")
    FText GetDisplayNameHelper() const;
    virtual FText GetDisplayNameHelper_Implementation() const;

    UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category="Live Link Device")
    EDeviceHealth GetDeviceHealthHelper() const;
    virtual EDeviceHealth GetDeviceHealthHelper_Implementation() const;

    UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category="Live Link Device")
    FText GetHealthTextHelper() const;
    virtual FText GetHealthTextHelper_Implementation() const;

    //~ Begin ILiveLinkDeviceCapability_Recording interface
    // These are already BlueprintNativeEvent so they can be overridden in Python
    virtual bool StartRecording_Implementation() override;
    virtual bool StopRecording_Implementation() override;
    virtual bool IsRecording_Implementation() const override;
    //~ End ILiveLinkDeviceCapability_Recording interface

protected:
    // Settings class to use (can be set by derived Python class)
    UPROPERTY(EditDefaultsOnly, Category="Live Link Device")
    TSubclassOf<ULiveLinkDeviceSettings> SettingsClass;

    UPROPERTY(BlueprintReadWrite, Category="Live Link Device")
	bool bIsRecording = false;

    UPROPERTY(BlueprintReadWrite, Category="Live Link Device")
	FString Slate;

    UPROPERTY(BlueprintReadWrite, Category="Live Link Device")
	FString Take;
};
