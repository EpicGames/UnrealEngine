// Copyright Epic Games, Inc. All Rights Reserved.
#include "LiveLinkGenericRecordingDeviceModule.h"

#define LOCTEXT_NAMESPACE "FLiveLinkGenericRecordingDeviceModule"

void FLiveLinkGenericRecordingDeviceModule::StartupModule()
{
	UE_LOGF(LogTemp, Verbose, "LiveLinkGenericRecordingDevice module started");
}

void FLiveLinkGenericRecordingDeviceModule::ShutdownModule()
{
	UE_LOGF(LogTemp, Verbose, "LiveLinkGenericRecordingDevice module shutdown");
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FLiveLinkGenericRecordingDeviceModule, LiveLinkGenericRecordingDevice)
